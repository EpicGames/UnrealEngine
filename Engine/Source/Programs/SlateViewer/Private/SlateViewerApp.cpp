// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateViewerApp.h"
#include "RequiredProgramMainCPPInclude.h"
#include "Widgets/Testing/STestSuite.h"
#include "Widgets/Testing/SStarshipSuite.h"
#include "ISourceCodeAccessModule.h"
#include "Widgets/Testing/SPerfSuite.h"
#include "Widgets/Docking/SDockTab.h"
#include "SWebBrowser.h"
#include "Framework/Application/SlateApplication.h"
#include "IWebBrowserWindow.h"
#include "IWebBrowserPopupFeatures.h"
#include "Stats/StatsSystem.h"
#include "WebBrowserModule.h"

#include "Styling/StarshipCoreStyle.h"


IMPLEMENT_APPLICATION(SlateViewer, "SlateViewer");

#define LOCTEXT_NAMESPACE "SlateViewer"

namespace WorkspaceMenu
{
	TSharedRef<FWorkspaceItem> DeveloperMenu = FWorkspaceItem::NewGroup(LOCTEXT("DeveloperMenu", "Developer"));
}


int RunSlateViewer( const TCHAR* CommandLine )
{

	FTaskTagScope TaskTagScope(ETaskTag::EGameThread);	

	// start up the main loop
	GEngineLoop.PreInit(CommandLine);

	// Make sure all UObject classes are registered and default properties have been initialized
	ProcessNewlyLoadedUObjects();
	
	// Tell the module manager it may now process newly-loaded UObjects when new C++ modules are loaded
	FModuleManager::Get().StartProcessingNewlyLoadedObjects();


	// crank up a normal Slate application using the platform's standalone renderer
	FSlateApplication::InitializeAsStandaloneApplication(GetStandardStandaloneRenderer());

	FSlateApplication::InitHighDPI(true);

	// Load the source code access module
	ISourceCodeAccessModule& SourceCodeAccessModule = FModuleManager::LoadModuleChecked<ISourceCodeAccessModule>( FName( "SourceCodeAccess" ) );
	
	// Manually load in the source code access plugins, as standalone programs don't currently support plugins.
#if PLATFORM_MAC
	IModuleInterface& XCodeSourceCodeAccessModule = FModuleManager::LoadModuleChecked<IModuleInterface>( FName( "XCodeSourceCodeAccess" ) );
	SourceCodeAccessModule.SetAccessor(FName("XCodeSourceCodeAccess"));
#elif PLATFORM_WINDOWS
	IModuleInterface& VisualStudioSourceCodeAccessModule = FModuleManager::LoadModuleChecked<IModuleInterface>( FName( "VisualStudioSourceCodeAccess" ) );
	SourceCodeAccessModule.SetAccessor(FName("VisualStudioSourceCodeAccess"));
#endif

	// set the application name
	FGlobalTabmanager::Get()->SetApplicationTitle(LOCTEXT("AppTitle", "Slate Viewer"));
	FModuleManager::LoadModuleChecked<ISlateReflectorModule>("SlateReflector").RegisterTabSpawner(WorkspaceMenu::DeveloperMenu);

	FGlobalTabmanager::Get()->RegisterNomadTabSpawner("WebBrowserTab", FOnSpawnTab::CreateStatic(&SpawnWebBrowserTab))
		.SetDisplayName(LOCTEXT("WebBrowserTab", "Web Browser"));
	
	if (FParse::Param(FCommandLine::Get(), TEXT("perftest")))
	{
		// Bring up perf test
		SummonPerfTestSuite();
	}
	else
	{
		FGlobalTabmanager::Get()->SetApplicationTitle(LOCTEXT("AppTitleStarship", "Starship Slate Viewer"));
		FAppStyle::SetAppStyleSetName(FStarshipCoreStyle::GetCoreStyle().GetStyleSetName());
		RestoreStarshipSuite();	

		if (FParse::Param(FCommandLine::Get(), TEXT("testsuite")))
		{
			RestoreSlateTestSuite();
		}
	}


#if WITH_SHARED_POINTER_TESTS
	SharedPointerTesting::TestSharedPointer<ESPMode::NotThreadSafe>();
	SharedPointerTesting::TestSharedPointer<ESPMode::ThreadSafe>();
#endif


	// initialize the web browser module
	IWebBrowserModule::Get();
	
	// loop while the server does the rest
	while (!IsEngineExitRequested())
	{
		BeginExitIfRequested();

		FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread);
		UE::Stats::FStats::AdvanceFrame(false);
		FTSTicker::GetCoreTicker().Tick(FApp::GetDeltaTime());
		FSlateApplication::Get().PumpMessages();
		FSlateApplication::Get().Tick();		
		FPlatformProcess::Sleep(0.01f);
	
		GFrameCounter++;
	}

	FCoreDelegates::OnExit.Broadcast();
	FSlateApplication::Shutdown();
	FModuleManager::Get().UnloadModulesAtShutdown();

	GEngineLoop.AppPreExit();
	GEngineLoop.AppExit();

	return 0;
}

namespace
{

	TMap<TWeakPtr<IWebBrowserWindow>, TWeakPtr<SWindow>> BrowserWindowWidgets;

	bool HandleBeforePopup(FString Url, FString Target)
	{
		return false; // Allow any popup
	}

	bool HandleBrowserCloseWindow(const TWeakPtr<IWebBrowserWindow>& BrowserWindowPtr)
	{
		TSharedPtr<IWebBrowserWindow> BrowserWindow = BrowserWindowPtr.Pin();
		if(BrowserWindow.IsValid())
		{
			if(!BrowserWindow->IsClosing())
			{
				// If the browser is not set to close, we tell the browser to close which will call back into this handler function.
				BrowserWindow->CloseBrowser(false);
			}
			else
			{
				const TWeakPtr<SWindow>* FoundBrowserWindowWidgetPtr = BrowserWindowWidgets.Find(BrowserWindow);
				if(FoundBrowserWindowWidgetPtr != nullptr)
				{
					TSharedPtr<SWindow> FoundBrowserWindowWidget = FoundBrowserWindowWidgetPtr->Pin();
					if(FoundBrowserWindowWidget.IsValid())
					{
						FoundBrowserWindowWidget->RequestDestroyWindow();
					}
					BrowserWindowWidgets.Remove(BrowserWindow);
					return true;
				}
			}
		}

		return false;
	}

	bool HandleBrowserCreateWindow(const TWeakPtr<IWebBrowserWindow>& NewBrowserWindow, const TWeakPtr<IWebBrowserPopupFeatures>& PopupFeatures, TSharedPtr<SWindow> ParentWindow)
	{
		if(ParentWindow.IsValid())
		{
			TSharedPtr<IWebBrowserPopupFeatures> PopupFeaturesSP = PopupFeatures.Pin();
			check(PopupFeatures.IsValid())

			const int PosX = PopupFeaturesSP->IsXSet() ? PopupFeaturesSP->GetX() : 100;
			const int PosY = PopupFeaturesSP->IsYSet() ? PopupFeaturesSP->GetY() : 100;
			const FVector2D BrowserWindowPosition(PosX, PosY);

			const int Width = PopupFeaturesSP->IsWidthSet() ? PopupFeaturesSP->GetWidth() : 800;
			const int Height = PopupFeaturesSP->IsHeightSet() ? PopupFeaturesSP->GetHeight() : 600;
			const FVector2D BrowserWindowSize(Width, Height);

			const ESizingRule SizeingRule = PopupFeaturesSP->IsResizable() ? ESizingRule::UserSized : ESizingRule::FixedSize;

			TSharedPtr<IWebBrowserWindow> NewBrowserWindowSP = NewBrowserWindow.Pin();
			check(NewBrowserWindowSP.IsValid())

			TSharedRef<SWindow> BrowserWindowWidget =
				SNew(SWindow)
				.Title(LOCTEXT("WebBrowserWindow_Title", "Web Browser"))
				.ClientSize(BrowserWindowSize)
				.ScreenPosition(BrowserWindowPosition)
				.AutoCenter(EAutoCenter::None)
				.SizingRule(SizeingRule)
				.SupportsMaximize(SizeingRule != ESizingRule::FixedSize)
				.SupportsMinimize(SizeingRule != ESizingRule::FixedSize)
				.HasCloseButton(true)
				.IsInitiallyMaximized(PopupFeaturesSP->IsFullscreen())
				.LayoutBorder(FMargin(0));

			// Setup browser widget.
			TSharedPtr<SWebBrowser> BrowserWidget;
			{
				TSharedPtr<SVerticalBox> Contents;
				BrowserWindowWidget->SetContent(
					SNew(SBorder)
					.VAlign(VAlign_Fill)
					.HAlign(HAlign_Fill)
					.Padding(0)
					[
						SAssignNew(Contents, SVerticalBox)
					]);

				Contents->AddSlot()
					[
						SAssignNew(BrowserWidget, SWebBrowser, NewBrowserWindowSP)
						.ShowControls(PopupFeaturesSP->IsToolBarVisible())
						.ShowAddressBar(PopupFeaturesSP->IsLocationBarVisible())
						.OnCreateWindow(FOnCreateWindowDelegate::CreateStatic(&HandleBrowserCreateWindow, ParentWindow))
						.OnCloseWindow(FOnCloseWindowDelegate::CreateStatic(&HandleBrowserCloseWindow))
					];
			}

			// Setup some OnClose stuff.
			{
				struct FLocal
				{
					static void RequestDestroyWindowOverride(const TSharedRef<SWindow>& Window, TWeakPtr<IWebBrowserWindow> BrowserWindowPtr)
					{
						TSharedPtr<IWebBrowserWindow> BrowserWindow = BrowserWindowPtr.Pin();
						if(BrowserWindow.IsValid())
						{
							if(BrowserWindow->IsClosing())
							{
								FSlateApplicationBase::Get().RequestDestroyWindow(Window);
							}
							else
							{
								// Notify the browser window that we would like to close it.  On the CEF side, this will 
								//  result in a call to FWebBrowserHandler::DoClose only if the JavaScript onbeforeunload
								//  event handler allows it.
								BrowserWindow->CloseBrowser(false);
							}
						}
					}
				};

				BrowserWindowWidget->SetRequestDestroyWindowOverride(FRequestDestroyWindowOverride::CreateStatic(&FLocal::RequestDestroyWindowOverride, TWeakPtr<IWebBrowserWindow>(NewBrowserWindow)));
			}

			FSlateApplication::Get().AddWindowAsNativeChild(BrowserWindowWidget, ParentWindow.ToSharedRef());
			BrowserWindowWidget->BringToFront();
			FSlateApplication::Get().SetKeyboardFocus( BrowserWidget, EFocusCause::SetDirectly );

			BrowserWindowWidgets.Add(NewBrowserWindow, BrowserWindowWidget);
			return true;
		}
		return false;
	}


}

TSharedRef<SDockTab> SpawnWebBrowserTab(const FSpawnTabArgs& Args)
{
	return SNew(SDockTab)
		.Label(LOCTEXT("WebBrowserTab", "Web Browser"))
		.ToolTipText(LOCTEXT("WebBrowserTabToolTip", "Switches to the Web Browser to test its features."))
		.TabRole(ETabRole::NomadTab)
		[
			SNew(SWebBrowser)
			.OnCreateWindow(FOnCreateWindowDelegate::CreateStatic(&HandleBrowserCreateWindow, Args.GetOwnerWindow()))
			.OnCloseWindow(FOnCloseWindowDelegate::CreateStatic(&HandleBrowserCloseWindow))
			.ParentWindow(Args.GetOwnerWindow())
		];
}

#undef LOCTEXT_NAMESPACE
