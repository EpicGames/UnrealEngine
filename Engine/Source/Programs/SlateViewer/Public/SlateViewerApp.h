// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ISlateReflectorModule.h"
#include "SlateViewer.h"
#include "Widgets/Docking/SDockTab.h"


/**
 * Run the SlateViewer .
 */
int RunSlateViewer(const TCHAR* Commandline);

/**
 * Spawn the contents of the web browser tab
 */
TSharedRef<SDockTab> SpawnWebBrowserTab(const FSpawnTabArgs& Args);
