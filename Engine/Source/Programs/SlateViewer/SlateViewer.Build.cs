// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class SlateViewer : ModuleRules
{
	public SlateViewer(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicIncludePathModuleNames.Add("Launch");

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"AppFramework",
				"Core",
				"ApplicationCore",
				"Projects",
				"Slate",
				"SlateCore",
				"StandaloneRenderer",
				"SourceCodeAccess",
				"WebBrowser",
			}
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"SlateReflector",
			}
		);

		DynamicallyLoadedModuleNames.AddRange(
			new string[] {
				"SlateReflector",
			}
		);

		if (Target.Platform == UnrealTargetPlatform.Mac)
		{
			PrivateDependencyModuleNames.Add("XCodeSourceCodeAccess");
			AddEngineThirdPartyPrivateStaticDependencies(Target, "CEF3");
		}
		else if (Target.Platform == UnrealTargetPlatform.Win64)
		{
			PrivateDependencyModuleNames.Add("VisualStudioSourceCodeAccess");
		}

		if (Target.IsInPlatformGroup(UnrealPlatformGroup.IOS))
		{
			PrivateDependencyModuleNames.AddRange(
                new string [] {
                    "NetworkFile",
                    "StreamingFile"
                }
            );
		}

		if (Target.IsInPlatformGroup(UnrealPlatformGroup.Linux))
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"UnixCommonStartup"
				}
			);
		}
	}
}
