// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.Collections.Generic;

[SupportedPlatforms(UnrealPlatformClass.Desktop)]
[SupportedPlatforms("IOS")]
[SupportedPlatforms("Android")]
public class SlateViewerTarget : TargetRules
{
	public SlateViewerTarget(TargetInfo Target) : base(Target)
	{
		Type = TargetType.Program;
		LinkType = TargetLinkType.Monolithic;

		LaunchModuleName = "SlateViewer";
		if (bBuildEditor)
		{
			ExtraModuleNames.Add("EditorStyle");
		}

		bBuildDeveloperTools = false;

		// SlateViewer doesn't ever compile with the engine linked in
		bCompileAgainstEngine = false;

		// We need CoreUObject compiled in as the source code access module requires it
		bCompileAgainstCoreUObject = true;

		// SlateViewer.exe has no exports, so no need to verify that a .lib and .exp file was emitted by
		// the linker.
		bHasExports = false;

		// Make sure to get all code in SlateEditorStyle compiled in
        bBuildDeveloperTools = true;
	}
}
